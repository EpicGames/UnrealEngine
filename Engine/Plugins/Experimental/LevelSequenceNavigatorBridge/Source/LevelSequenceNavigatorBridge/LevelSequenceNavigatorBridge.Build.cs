// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LevelSequenceNavigatorBridge : ModuleRules
{
	public LevelSequenceNavigatorBridge(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicIncludePaths.AddRange(new string[] {

		});

		PrivateIncludePaths.AddRange(new string[] {

		});

		PublicDependencyModuleNames.AddRange(new string[] {
			"Core"
		});

		PrivateDependencyModuleNames.AddRange(new string[] {
			"CoreUObject",
			"Engine",
			"LevelSequence",
			"MovieScene",
			"SequenceNavigator",
			"Sequencer",
			"SequencerCore",
			"Slate",
			"SlateCore"
		});

		DynamicallyLoadedModuleNames.AddRange(new string[] {

		});
	}
}
