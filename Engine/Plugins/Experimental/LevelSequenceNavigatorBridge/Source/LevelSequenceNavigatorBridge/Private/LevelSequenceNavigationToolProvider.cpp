// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceNavigationToolProvider.h"
#include "Columns/NavigationToolColorColumn.h"
#include "Columns/NavigationToolColumnExtender.h"
#include "Columns/NavigationToolCommentColumn.h"
#include "Columns/NavigationToolHBiasColumn.h"
#include "Columns/NavigationToolIdColumn.h"
#include "Columns/NavigationToolInTimeColumn.h"
#include "Columns/NavigationToolItemsColumn.h"
#include "Columns/NavigationToolLabelColumn.h"
#include "Columns/NavigationToolLengthColumn.h"
#include "Columns/NavigationToolLockColumn.h"
#include "Columns/NavigationToolMarkerVisibilityColumn.h"
#include "Columns/NavigationToolOutTimeColumn.h"
#include "Columns/NavigationToolPlayheadColumn.h"
#include "Columns/NavigationToolRevisionControlColumn.h"
#include "Columns/NavigationToolStartFrameOffsetColumn.h"
#include "Columns/NavigationToolTakeColumn.h"
#include "Columns/NavigationToolDeactiveStateColumn.h"
#include "Filters/Filters/NavigationToolBuiltInFilterParams.h"
#include "INavigationTool.h"
#include "ISequencer.h"
#include "ISourceControlModule.h"
#include "Items/NavigationToolSequence.h"
#include "LevelSequence.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "NavigationToolDefines.h"
#include "NavigationToolSettings.h"

#define LOCTEXT_NAMESPACE "LevelSequenceNavigationToolProvider"

const FName FLevelSequenceNavigationToolProvider::Identifier = TEXT("LevelSequence");

const FText FLevelSequenceNavigationToolProvider::AnimationColumnViewName = LOCTEXT("AnimationColumnViewName", "Animation");

FLevelSequenceNavigationToolProvider::FLevelSequenceNavigationToolProvider()
	: ToolCommands(MakeShared<FUICommandList>())
{
}

FName FLevelSequenceNavigationToolProvider::GetIdentifier() const
{
	return Identifier;
}

TSet<TSubclassOf<UMovieSceneSequence>> FLevelSequenceNavigationToolProvider::GetSupportedSequenceClasses() const
{
	return { ULevelSequence::StaticClass() };
}

FText FLevelSequenceNavigationToolProvider::GetDefaultColumnView() const
{
	return AnimationColumnViewName;
}

void FLevelSequenceNavigationToolProvider::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	InCommandList->Append(ToolCommands);
}

void FLevelSequenceNavigationToolProvider::OnActivate()
{
	// @TODO: Move from core Navigation Tool
	/*ExtendToolToolBar();
	ExtendToolItemContextMenu();

	DragDropInitializedDelegate = FNavigationToolExtender::OnItemDragDropOpInitialized()
		.AddSP(this, &FAvaNavigationToolProvider::ExtendToolItemDragDropOp);*/
}

void FLevelSequenceNavigationToolProvider::OnDeactivate()
{
	// @TODO: Move from core Navigation Tool
	/*RemoveToolToolBarExtension();
	RemoveToolItemContextMenuExtension();

	FNavigationToolExtender::OnItemDragDropOpInitialized()
		.Remove(DragDropInitializedDelegate);*/
}

void FLevelSequenceNavigationToolProvider::OnExtendColumns(UE::SequenceNavigator::FNavigationToolColumnExtender& OutExtender)
{
	using namespace UE::SequenceNavigator;

	OutExtender.AddColumn<FNavigationToolPlayheadColumn>();
	OutExtender.AddColumn<FNavigationToolDeactiveStateColumn>();
	OutExtender.AddColumn<FNavigationToolMarkerVisibilityColumn>();
	OutExtender.AddColumn<FNavigationToolLockColumn>();
	OutExtender.AddColumn<FNavigationToolColorColumn>();
	OutExtender.AddColumn<FNavigationToolLabelColumn>();
	OutExtender.AddColumn<FNavigationToolIdColumn>();
	OutExtender.AddColumn<FNavigationToolItemsColumn>();
	OutExtender.AddColumn<FNavigationToolInTimeColumn>();
	OutExtender.AddColumn<FNavigationToolOutTimeColumn>();
	OutExtender.AddColumn<FNavigationToolLengthColumn>();
	OutExtender.AddColumn<FNavigationToolHBiasColumn>();
	OutExtender.AddColumn<FNavigationToolStartFrameOffsetColumn>();
	OutExtender.AddColumn<FNavigationToolTakeColumn>();
	OutExtender.AddColumn<FNavigationToolCommentColumn>();

	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	if (SourceControlModule.IsEnabled())
	{
		OutExtender.AddColumn<FNavigationToolRevisionControlColumn>();
	}

	FNavigationToolProvider::OnExtendColumns(OutExtender);
}

void FLevelSequenceNavigationToolProvider::OnExtendColumnViews(TSet<FNavigationToolColumnView>& OutColumnViews)
{
	using namespace UE::SequenceNavigator;

	FNavigationToolColumnView& AnimationColumnView = OutColumnViews
		.FindOrAdd(FNavigationToolColumnView(AnimationColumnViewName));

	AnimationColumnView.VisibleColumns =
		{
			FNavigationToolPlayheadColumn::StaticColumnId(),
			FNavigationToolColorColumn::StaticColumnId(),
			FNavigationToolLabelColumn::StaticColumnId(),
			FNavigationToolItemsColumn::StaticColumnId(),
			FNavigationToolInTimeColumn::StaticColumnId(),
			FNavigationToolOutTimeColumn::StaticColumnId(),
			FNavigationToolHBiasColumn::StaticColumnId()
		};

	FNavigationToolProvider::OnExtendColumnViews(OutColumnViews);
}

void FLevelSequenceNavigationToolProvider::OnExtendItemChildren(UE::SequenceNavigator::INavigationTool& InTool
	, const UE::SequenceNavigator::FNavigationToolViewModelPtr& InParentItem
	, TArray<UE::SequenceNavigator::FNavigationToolViewModelWeakPtr>& OutWeakChildren
	, const bool bInRecursive)
{
	using namespace UE::SequenceNavigator;

	FNavigationToolProvider::OnExtendItemChildren(InTool, InParentItem, OutWeakChildren, bInRecursive);

	// Only extending root item
	if (InParentItem->GetItemId() != FNavigationToolItemId::RootId)
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = InTool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UMovieSceneSequence* const RootSequence = Sequencer->GetRootMovieSceneSequence();
	if (!RootSequence)
	{
		return;
	}

	const TSharedRef<FLevelSequenceNavigationToolProvider> ProviderRef = SharedThis(this);

	const FNavigationToolViewModelPtr NewItem = InTool.FindOrAdd<FNavigationToolSequence>(ProviderRef
		, InParentItem, RootSequence, nullptr, 0);
	const FNavigationToolItemFlagGuard Guard(NewItem, ENavigationToolItemFlags::IgnorePendingKill);
	OutWeakChildren.Add(NewItem);
	if (bInRecursive)
	{
		NewItem->FindChildren(OutWeakChildren, bInRecursive);
	}
}

void FLevelSequenceNavigationToolProvider::OnExtendBuiltInFilters(TArray<FNavigationToolBuiltInFilterParams>& OutFilterParams)
{
	OutFilterParams.Add(FNavigationToolBuiltInFilterParams::CreateSequenceFilter());
	OutFilterParams.Add(FNavigationToolBuiltInFilterParams::CreateTrackFilter());
	OutFilterParams.Add(FNavigationToolBuiltInFilterParams::CreateBindingFilter());
	OutFilterParams.Add(FNavigationToolBuiltInFilterParams::CreateMarkerFilter());

	FNavigationToolProvider::OnExtendBuiltInFilters(OutFilterParams);
}

FMovieSceneEditorData* FLevelSequenceNavigationToolProvider::GetRootMovieSceneEditorData(const UE::SequenceNavigator::INavigationTool& InTool) const
{
	const TSharedPtr<ISequencer> Sequencer = InTool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return nullptr;
	}

	UMovieSceneSequence* const RootMovieSceneSequence = Sequencer->GetRootMovieSceneSequence();
	if (!RootMovieSceneSequence)
	{
		return nullptr;
	}

	UMovieScene* const RootMovieScene = RootMovieSceneSequence->GetMovieScene();
	if (!RootMovieScene)
	{
		return nullptr;
	}

	return &RootMovieScene->GetEditorData();
}

FNavigationToolSaveState* FLevelSequenceNavigationToolProvider::GetSaveState(const UE::SequenceNavigator::INavigationTool& InTool) const
{
	if (FMovieSceneEditorData* const EditorData = GetRootMovieSceneEditorData(InTool))
	{
		return &EditorData->NavigationToolState;
	}
	return nullptr;
}

void FLevelSequenceNavigationToolProvider::SetSaveState(const UE::SequenceNavigator::INavigationTool& InTool
	, const FNavigationToolSaveState& InSaveState) const
{
	if (FMovieSceneEditorData* const EditorData = GetRootMovieSceneEditorData(InTool))
	{
		EditorData->NavigationToolState = InSaveState;
	}
}

#undef LOCTEXT_NAMESPACE
