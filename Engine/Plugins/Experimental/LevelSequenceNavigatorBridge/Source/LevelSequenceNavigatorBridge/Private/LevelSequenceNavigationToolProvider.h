// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "Providers/NavigationToolProvider.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class FUICommandList;
class UMovieSceneSequence;
struct FMovieSceneEditorData;
struct FNavigationToolBuiltInFilterParams;
struct FNavigationToolColumnView;
struct FNavigationToolSaveState;

namespace UE::SequenceNavigator
{
	class FNavigationToolColumnExtender;
	class FNavigationToolItem;
	class INavigationTool;
}

class FLevelSequenceNavigationToolProvider : public UE::SequenceNavigator::FNavigationToolProvider
{
public:
	static const FName Identifier;

	static const FText AnimationColumnViewName;

	FLevelSequenceNavigationToolProvider();
	virtual ~FLevelSequenceNavigationToolProvider() override = default;

	//~ Begin INavigationToolProvider

	virtual FName GetIdentifier() const override;

	virtual TSet<TSubclassOf<UMovieSceneSequence>> GetSupportedSequenceClasses() const override;

	virtual FText GetDefaultColumnView() const override;

	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) override;

	virtual void OnActivate() override;
	virtual void OnDeactivate() override;

	virtual void OnExtendColumns(UE::SequenceNavigator::FNavigationToolColumnExtender& OutExtender) override;
	virtual void OnExtendColumnViews(TSet<FNavigationToolColumnView>& OutColumnViews) override;
	virtual void OnExtendItemChildren(UE::SequenceNavigator::INavigationTool& InTool
		, const UE::SequenceNavigator::FNavigationToolViewModelPtr& InParentItem
		, TArray<UE::SequenceNavigator::FNavigationToolViewModelWeakPtr>& OutWeakChildren
		, const bool bInRecursive) override;
	virtual void OnExtendBuiltInFilters(TArray<FNavigationToolBuiltInFilterParams>& OutFilterParams) override;

	virtual FNavigationToolSaveState* GetSaveState(const UE::SequenceNavigator::INavigationTool& InTool) const override;
	virtual void SetSaveState(const UE::SequenceNavigator::INavigationTool& InTool
		, const FNavigationToolSaveState& InSaveState) const override;

	//~ End INavigationToolProvider

private:
	FMovieSceneEditorData* GetRootMovieSceneEditorData(const UE::SequenceNavigator::INavigationTool& InTool) const;

	TSharedRef<FUICommandList> ToolCommands;
};
