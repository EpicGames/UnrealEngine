// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceNavigatorBridgeModule.h"
#include "ISequencer.h"
#include "ISequencerModule.h"
#include "LevelSequenceNavigationToolProvider.h"
#include "NavigationToolExtender.h"

#define LOCTEXT_NAMESPACE "LevelSequenceNavigatorBridgeModule"

using namespace UE::SequenceNavigator;

void FLevelSequenceNavigatorBridgeModule::StartupModule()
{
	if (ISequencerModule* const SequencerModule = FModuleManager::LoadModulePtr<ISequencerModule>(TEXT("Sequencer")))
	{
		SequencerCreatedHandle = SequencerModule->RegisterOnSequencerCreated(
			FOnSequencerCreated::FDelegate::CreateRaw(this, &FLevelSequenceNavigatorBridgeModule::OnSequencerCreated));
	}
}

void FLevelSequenceNavigatorBridgeModule::ShutdownModule()
{
	if (ISequencerModule* const SequencerModule = FModuleManager::LoadModulePtr<ISequencerModule>(TEXT("Sequencer")))
	{
		SequencerModule->UnregisterOnSequencerCreated(SequencerCreatedHandle);
		SequencerCreatedHandle.Reset();
	}
}

void FLevelSequenceNavigatorBridgeModule::OnSequencerCreated(const TSharedRef<ISequencer> InSequencer)
{
	const FName ToolId = FNavigationToolExtender::GetToolInstanceId(*InSequencer);

	const TSharedPtr<FNavigationToolProvider> FoundProvider = FNavigationToolExtender::FindToolProvider(ToolId
		, FLevelSequenceNavigationToolProvider::Identifier);
	if (!FoundProvider.IsValid())
	{
		NavigationToolProvider = MakeShared<FLevelSequenceNavigationToolProvider>();
		if (NavigationToolProvider->IsSequenceSupported(InSequencer->GetRootMovieSceneSequence()))
		{
			FNavigationToolExtender::RegisterToolProvider(InSequencer, NavigationToolProvider.ToSharedRef());
		}
		else
		{
			NavigationToolProvider.Reset();
		}
	}

	SequencerClosedHandle = InSequencer->OnCloseEvent().AddRaw(this, &FLevelSequenceNavigatorBridgeModule::OnSequencerClosed);
}

void FLevelSequenceNavigatorBridgeModule::OnSequencerClosed(const TSharedRef<ISequencer> InSequencer)
{
	if (!NavigationToolProvider.IsValid())
	{
		return;
	}

	const FName ToolId = FNavigationToolExtender::GetToolInstanceId(*InSequencer);

	if (FNavigationToolExtender::UnregisterToolProvider(ToolId, NavigationToolProvider->GetIdentifier()))
	{
		NavigationToolProvider.Reset();
		SequencerClosedHandle.Reset();
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FLevelSequenceNavigatorBridgeModule, LevelSequenceNavigatorBridge)
