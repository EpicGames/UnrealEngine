// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

class FLevelSequenceNavigationToolProvider;
class ISequencer;

class FLevelSequenceNavigatorBridgeModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

private:
	void OnSequencerCreated(const TSharedRef<ISequencer> InSequencer);
	void OnSequencerClosed(const TSharedRef<ISequencer> InSequencer);

	FDelegateHandle SequencerCreatedHandle;
	FDelegateHandle SequencerClosedHandle;

	/** The provider that supplies data and extends the Sequence Navigator */
	TSharedPtr<FLevelSequenceNavigationToolProvider> NavigationToolProvider;
};
